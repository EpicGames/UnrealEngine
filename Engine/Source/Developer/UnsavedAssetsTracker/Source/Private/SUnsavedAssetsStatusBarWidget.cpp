// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUnsavedAssetsStatusBarWidget.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "UnsavedAssetsTracker.h"

#define LOCTEXT_NAMESPACE "UnsavedAssetsTracker"

void SUnsavedAssetsStatusBarWidget::Construct(const FArguments& InArgs, TSharedPtr<FUnsavedAssetsTracker> InUnsavedAssetTracker)
{
	UnsavedAssetTracker = MoveTemp(InUnsavedAssetTracker);

	ChildSlot
	[
		SNew(SButton)
		.ContentPadding(FMargin(6.0f,0.0f))
		.ToolTipText_Lambda([this]() { return GetStatusBarTooltip(); })
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(InArgs._OnClicked)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image_Lambda([this](){ return GetStatusBarIcon(); })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(5, 0, 0, 0))
			[
				SNew(STextBlock)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
				.Text_Lambda([this]() { return GetStatusBarText(); })
			]
		]
	];
}

const FSlateBrush* SUnsavedAssetsStatusBarWidget::GetStatusBarIcon() const
{
	if (UnsavedAssetTracker->GetUnsavedAssetNum() == 0)
	{
		return FAppStyle::GetBrush("Assets.AllSaved");
	}
	else if (UnsavedAssetTracker->GetWarningNum() > 0)
	{
		return FAppStyle::GetBrush("Assets.UnsavedWarning");
	}
	else
	{
		return FAppStyle::GetBrush("Assets.Unsaved");
	}
}

FText SUnsavedAssetsStatusBarWidget::GetStatusBarText() const
{
	if (UnsavedAssetTracker->GetUnsavedAssetNum() == 0)
	{
		return LOCTEXT("All_Saved", "All Saved");
	}
	return FText::Format(LOCTEXT("Unsaved Assets", "{0} Unsaved"), UnsavedAssetTracker->GetUnsavedAssetNum());
}

FText SUnsavedAssetsStatusBarWidget::GetStatusBarTooltip() const
{
	if (UnsavedAssetTracker->GetUnsavedAssetNum() == 0)
	{
		return LOCTEXT("Assets_All_Saved_Tooltip", "All assets in the project have been saved.");
	}
	else if (UnsavedAssetTracker->GetWarningNum() > 0)
	{
		return LOCTEXT("Asset_Unsaved_Warning_Tooltip", "Warning: There are currently unsaved assets that have a conflict in Revision Control");
	}
	else
	{
		const int32 UnsavedAssetNum = UnsavedAssetTracker->GetUnsavedAssetNum();
		if (UnsavedAssetNum <= 1)
		{
			return FText::Format(LOCTEXT("Single_Asset_Unsaved_Tooltip", "There is currently {0} asset that is unsaved in this project. Press to initiate save."), UnsavedAssetNum);
		}
		else
		{
			return FText::Format(LOCTEXT("Asset_Unsaved_Tooltip", "There are currently {0} assets that are unsaved in this project. Press to initiate save."), UnsavedAssetNum);
		}
	}
}

#undef LOCTEXT_NAMESPACE
