// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FUnsavedAssetsTracker;

/**
 * Tracks assets that has in-memory modification not saved to disk yet and checks
 * the source control states of those assets when a source control provider is available.
 */
class SUnsavedAssetsStatusBarWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUnsavedAssetsStatusBarWidget)
	{}
		/** Event fired when the status bar button is clicked */
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs, TSharedPtr<FUnsavedAssetsTracker> InUnsavedAssetTracker);

private:
	const FSlateBrush* GetStatusBarIcon() const;
	FText GetStatusBarText() const;
	FText GetStatusBarTooltip() const;

private:
	TSharedPtr<FUnsavedAssetsTracker> UnsavedAssetTracker;
};

