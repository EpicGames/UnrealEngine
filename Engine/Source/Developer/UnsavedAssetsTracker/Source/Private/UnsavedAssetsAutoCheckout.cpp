// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsavedAssetsAutoCheckout.h"

#include "ISourceControlModule.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Containers/Ticker.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "SourceControlOperations.h"
#include "UnsavedAssetsTrackerModule.h"
#include "UncontrolledChangelistsModule.h"

FUnsavedAssetsAutoCheckout::FUnsavedAssetsAutoCheckout(FUnsavedAssetsTrackerModule* Module)
	: bProcessCheckoutBatchPending(false)
	, CheckOutBatch()
	, CheckOutOperation(ISourceControlOperation::Create<FCheckOut>())
{
	Module->OnUnsavedAssetAdded.AddRaw(this, &FUnsavedAssetsAutoCheckout::OnAsyncCheckout);
	AsyncCheckoutComplete.BindRaw(this, &FUnsavedAssetsAutoCheckout::OnAsyncCheckoutComplete);
}

FUnsavedAssetsAutoCheckout::~FUnsavedAssetsAutoCheckout()
{
}

void FUnsavedAssetsAutoCheckout::OnAsyncCheckout(const FString& AbsoluteAssetFilepath)
{
	// Check if auto checkout is enabled.
	const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
	if (!Settings->GetAutomaticallyCheckoutOnAssetModification())
	{
		return;
	}

	// Check if SourceControl is enabled and available.
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (!SourceControlModule.IsEnabled())
	{
		return;
	}
	else
	{
		bool bIsAvailable = SourceControlModule.GetProvider().IsAvailable();
		if (!bIsAvailable)
		{
			// The UncontrolledChangelists module relies on the 'checkout' to make the files locally writable
			// and able to reconcile later. So if used and enabled, allow the 'checkout' to continue.
			bool bUncontrolledChangelistsUsed = SourceControlModule.GetProvider().UsesUncontrolledChangelists();
			bool bUncontrolledChangelistsEnabled = FUncontrolledChangelistsModule::Get().IsEnabled();
			
			bool bUncontrolledChangelistsRequiresCheckout = bUncontrolledChangelistsUsed && bUncontrolledChangelistsEnabled;
			if (!bUncontrolledChangelistsRequiresCheckout)
			{
				return;
			}
		}
	}

	// Check if SourceControl is using checkout.
	bool bUsesCheckout = SourceControlModule.GetProvider().UsesCheckout();
	if (!bUsesCheckout)
	{
		return;
	}

	// Check if SourceControl has already checked out this file for us.
	// If it's checked out by someone else, we want this to proceed, so the operation can be auto-undo'd.
	FSourceControlStatePtr State = SourceControlModule.GetProvider().GetState(AbsoluteAssetFilepath, EStateCacheUsage::Use);
	if (State.IsValid() && State->IsCheckedOut())
	{
		return;
	}
	// Check if this is an added file, in which case it cannot be checked out or does not need to be checked out.
	if (State.IsValid() && State->IsAdded())
	{
		return;
	}

	// Add to CheckOutBatch.
	// Why are we batching? Because moving a large prefab (1000+ source files) would result in
	// an equal amount of FCheckOut operations which didn't scale and locked the editor due to
	// worker thread exhaustion.
	CheckOutBatch.Add(AbsoluteAssetFilepath);

	// Broadcast that this file is potentially going to be automatically checked out.
	// This will be followed up by either a Success/Failure/Cancel notification.
	FUnsavedAssetsTrackerModule::Get().PreUnsavedAssetAutoCheckout.Broadcast(AbsoluteAssetFilepath, CheckOutOperation);

	// If no batch process tick is pending, queue one.
	if (!bProcessCheckoutBatchPending)
	{
		bProcessCheckoutBatchPending = true;

		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateSP(this, &FUnsavedAssetsAutoCheckout::OnProcessCheckoutBatch)
		);
	}
}

void FUnsavedAssetsAutoCheckout::OnAsyncCheckoutComplete(const FSourceControlOperationRef& InCheckOutOperation, ECommandResult::Type Result)
{
	if (TArray<FString>* Files = OperationToFiles.Find(InCheckOutOperation))
	{
		switch (Result)
		{
		case ECommandResult::Succeeded:
			NotifySuccess(MakeArrayView(*Files), InCheckOutOperation);
			break;
		case ECommandResult::Failed:
			NotifyFailure(MakeArrayView(*Files), InCheckOutOperation);
			break;
		case ECommandResult::Cancelled:
			NotifyCancel(MakeArrayView(*Files), InCheckOutOperation);
			break;
		default:
			checkNoEntry();
			break;
		}

		OperationToFiles.Remove(InCheckOutOperation);
	}
}

bool FUnsavedAssetsAutoCheckout::OnProcessCheckoutBatch(float)
{
	TArray<FString> FilesToCheckout;
	FilesToCheckout.Reserve(CheckOutBatch.Num());

	TArray<FString> FilesToCancel;
	FilesToCancel.Reserve(CheckOutBatch.Num());

	// Determine which files need not be checked out and can thus be canceled.
	for (const FString& File : CheckOutBatch)
	{
		// Why are we checking for file exists?
		// The OnUnsavedAssetAdded delegate triggers when dragging an asset in the viewport as well,
		// before it's even placed in the scene or saved to disk. There's no point in attempting
		// to do an FCheckOut for them.
		if (!FPaths::FileExists(File))
		{
			FilesToCancel.Add(File);
			continue;
		}

		// Why are we checking if the package is still loaded and dirty?
		// Some packages become temporarily dirty, for example during world destruction / world recreation.
		// By the time we try to check them out they may no longer be dirty or even loaded.
		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(File, PackageName))
		{
			UPackage* Package = FindPackage(nullptr, *PackageName);
			if (!Package || !Package->IsDirty())
			{
				FilesToCancel.Add(File);
				continue;
			}
		}

		// If not canceled, it should be checked out.
		FilesToCheckout.Add(File);
	}

	// Call the Cancel delegate for those that no longer need to be checked out.
	if (FilesToCancel.Num() > 0)
	{
		NotifyCancel(FilesToCancel, CheckOutOperation);
	}
	// Call the Success/Failure/Cancel delegate for others by triggering a checkout operation.
	if (FilesToCheckout.Num() > 0)
	{
		TriggerAsyncCheckout(FilesToCheckout, CheckOutOperation);
	}

	// Reset now that all files in this batch are either cancelled or pending checkout.
	CheckOutBatch.Empty();
	CheckOutOperation = ISourceControlOperation::Create<FCheckOut>();
	bProcessCheckoutBatchPending = false;

	return false; // One shot.
}

void FUnsavedAssetsAutoCheckout::TriggerAsyncCheckout(const TArray<FString>& FilesToCheckout, const FSourceControlOperationRef& Operation)
{
	ensure(FilesToCheckout.Num() > 0);

	OperationToFiles.Add(Operation, FilesToCheckout);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	SourceControlProvider.Execute(Operation, FilesToCheckout, EConcurrency::Asynchronous, AsyncCheckoutComplete);
}

void FUnsavedAssetsAutoCheckout::NotifySuccess(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation)
{
	for (const FString& File : Files)
	{
		FUnsavedAssetsTrackerModule::Get().PostUnsavedAssetAutoCheckout.Broadcast(File, Operation);
	}
}

void FUnsavedAssetsAutoCheckout::NotifyFailure(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation)
{
	for (const FString& File : Files)
	{
		FUnsavedAssetsTrackerModule::Get().PostUnsavedAssetAutoCheckoutFailure.Broadcast(File, Operation);
	}
}

void FUnsavedAssetsAutoCheckout::NotifyCancel(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation)
{
	for (const FString& File : Files)
	{
		FUnsavedAssetsTrackerModule::Get().PostUnsavedAssetAutoCheckoutCancel.Broadcast(File, Operation);
	}
}