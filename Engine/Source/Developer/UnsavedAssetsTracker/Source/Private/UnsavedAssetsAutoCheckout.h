// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/Map.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

class ISourceControlOperation;
class FUnsavedAssetsTrackerModule;

/**
 * Uses the source control provider to attempt automatic checkouts of unsaved assets.
 */
class FUnsavedAssetsAutoCheckout : public TSharedFromThis<FUnsavedAssetsAutoCheckout>
{
public:
	explicit FUnsavedAssetsAutoCheckout(FUnsavedAssetsTrackerModule* Module);
	~FUnsavedAssetsAutoCheckout();

private:
	void OnAsyncCheckout(const FString& AbsoluteAssetFilepath);
	
	FSourceControlOperationComplete AsyncCheckoutComplete;
	void OnAsyncCheckoutComplete(const FSourceControlOperationRef&, ECommandResult::Type);

	/* To process checkouts in a single batch once per tick */
	bool OnProcessCheckoutBatch(float);
	bool bProcessCheckoutBatchPending;

	void TriggerAsyncCheckout(const TArray<FString>& FilesToCheckout, const FSourceControlOperationRef& Operation);

	void NotifySuccess(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation);
	void NotifyFailure(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation);
	void NotifyCancel(TConstArrayView<FString> Files, const FSourceControlOperationRef& Operation);

	TMap<FSourceControlOperationRef, TArray<FString>> OperationToFiles;

	TSet<FString> CheckOutBatch;
	FSourceControlOperationRef CheckOutOperation;
};
