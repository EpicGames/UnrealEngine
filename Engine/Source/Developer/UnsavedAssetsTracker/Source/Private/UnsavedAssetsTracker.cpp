// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsavedAssetsTracker.h"

#include "UnsavedAssetsTrackerModule.h"
#include "Algo/Transform.h"
#include "AssetToolsModule.h"
#include "FileHelpers.h"
#include "Logging/LogMacros.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlFileStatusMonitor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/TransBuffer.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "UnrealEdGlobals.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeLock.h"
#include "LevelEditor.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "IAssetTools.h"
#include "UObject/UObjectGlobals.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#define LOCTEXT_NAMESPACE "UnsavedAssetsTracker"

DEFINE_LOG_CATEGORY_STATIC(LogUnsavedAssetsTracker, Log, All);

namespace
{
static bool bSyncUnsavedAssetInfoToTEDS = true;
FAutoConsoleVariableRef CVarSyncUnsavedInformation(
	TEXT("TEDS.RevisionControl.SyncUnsavedInformation"),
	bSyncUnsavedAssetInfoToTEDS,
	TEXT("Automatically sync information about unsaved assets to TEDS")
);

FString GetHumanFriendlyAssetName(const UPackage* Package)
{
	FName AssetName;
	FName OwnerName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	// Lookup for the first asset in the package
	UObject* FoundAsset = nullptr;
	ForEachObjectWithPackage(Package, [&FoundAsset](UObject* InnerObject)
	{
		if (InnerObject->IsAsset())
		{
			if (FAssetData::IsUAsset(InnerObject))
			{
				// If we found the primary asset, use it
				FoundAsset = InnerObject;
				return false;
			}
			// Otherwise, keep the first found asset but keep looking for a primary asset
			if (!FoundAsset)
			{
				FoundAsset = InnerObject;
			}
		}
		return true;
	}, EGetObjectsFlags::None);

	if (FoundAsset)
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(FoundAsset->GetClass());
		if (AssetTypeActions.IsValid())
		{
			AssetName = *AssetTypeActions.Pin()->GetObjectDisplayName(FoundAsset);
		}
		else
		{
			AssetName = FoundAsset->GetFName();
		}

		OwnerName = FoundAsset->GetOutermostObject()->GetFName();
	}

	// Last resort, display the package name
	if (AssetName == NAME_None)
	{
		AssetName = *FPackageName::GetShortName(Package->GetFName());
	}

	return AssetName.ToString();
}

bool IsPackageMeantToBeSaved(const UPackage* Package)
{
	// Ignore the packages that aren't meant to be persistent.
	if (Package->HasAnyFlags(RF_Transient) ||
		Package->HasAnyPackageFlags(PKG_CompiledIn) ||
		Package->HasAnyPackageFlags(PKG_PlayInEditor) ||
		Package == GetTransientPackage() ||
		FPackageName::IsMemoryPackage(Package->GetPathName()))
	{
		return false;
	}
	return true;
}

bool HasPackageWritePermissions(const UPackage* Package)
{
	// if we do not have write permission under the mount point for this package log an error in the message log to link to.
	FString PackageName = Package->GetName();
	return GUnrealEd->HasMountWritePermissionForPackage(PackageName);
}

// NOTE: This function is very similar to USourceControlHelpers::PackageFilename() but it doesn't call FindPackage() which can
//       crash the engine when auto-saving packages.
FString GetAbsolutePackageFilePathname(const UPackage* InPackage)
{
	auto GetPathnameInternal = [](const UPackage* Package)
	{
		FString PackageName = Package->GetName();
		FString Filename = Package->GetName();

		// Get the filename by finding it on disk first
		if (!FPackageName::DoesPackageExist(PackageName, &Filename))
		{
			// The package does not exist on disk, see if we can find it in memory and predict the file extension
			// Only do this if the supplied package name is valid
			const bool bIncludeReadOnlyRoots = false;
			if (FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots))
			{
				// This is a package in memory that has not yet been saved. Determine the extension and convert to a filename, if we do have the package, just assume normal asset extension
				const FString PackageExtension = Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
				Filename = FPackageName::LongPackageNameToFilename(PackageName, PackageExtension);
			}
		}

		return Filename;
	};

	return FPaths::ConvertRelativePathToFull(GetPathnameInternal(InPackage));
}


} // Anonoymous namespace


FUnsavedAssetsTracker::FUnsavedAssetsTracker()
{
	// Register for the package dirty state updated callback to catch packages that have been cleaned without being saved
	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FUnsavedAssetsTracker::OnPackageDirtyStateUpdated);

	// Register for the "MarkPackageDirty" callback to catch packages that have been modified and need to be saved
	UPackage::PackageMarkedDirtyEvent.AddRaw(this, &FUnsavedAssetsTracker::OnPackageMarkedDirty);

	// Register for the package modified callback to catch packages that have been saved
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FUnsavedAssetsTracker::OnPackageSaved);

	// Register for post gc to cleanup package list that might have been gced
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FUnsavedAssetsTracker::OnPostGarbageCollect);

	// Register to get notified when something gets deleted
	FEditorDelegates::OnPackageDeleted.AddRaw(this, &FUnsavedAssetsTracker::OnPackageDeleted);
	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddRaw(this, &FUnsavedAssetsTracker::OnActorDeleted);
	}

	// Hook to detect when a map is changed to refresh to catch when a temporary map is discarded.
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddRaw(this, &FUnsavedAssetsTracker::OnMapChanged);

	// Register in the ticker.
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FUnsavedAssetsTracker::Tick));

	// Hook to detect when a a world is renamed to to catch when a temporary map is saved with a new name.
	FWorldDelegates::OnPostWorldRename.AddRaw(this, &FUnsavedAssetsTracker::OnWorldPostRename);

	// Hook to detect when an Undo/Redo change the dirty state of a package.
	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
		TransBuffer->OnUndo().AddRaw(this, &FUnsavedAssetsTracker::OnUndo);
		TransBuffer->OnRedo().AddRaw(this, &FUnsavedAssetsTracker::OnRedo);
	}
}

FUnsavedAssetsTracker::~FUnsavedAssetsTracker()
{
	FTSTicker::RemoveTicker(TickerHandle);

	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
	UPackage::PackageMarkedDirtyEvent.RemoveAll(this);
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	FEditorDelegates::OnPackageDeleted.RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnMapChanged().RemoveAll(this);
	}

	FWorldDelegates::OnPostWorldRename.RemoveAll(this);

	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
		TransBuffer->OnUndo().RemoveAll(this);
		TransBuffer->OnRedo().RemoveAll(this);
	}
}

int32 FUnsavedAssetsTracker::GetUnsavedAssetNum() const
{
	check(UnsavedPackages.Num() == UnsavedAbsFilePathames.Num()); // UnsavedAbsFilePathames is use to reverse lookup UnsavedPackages, so they should have the same number of elements
	return UnsavedPackages.Num();
}

TArray<FString> FUnsavedAssetsTracker::GetUnsavedAssets() const
{
	TArray<FString> Pathnames;
	UnsavedAbsFilePathames.GetKeys(Pathnames);
	return Pathnames;
}

int32 FUnsavedAssetsTracker::GetWarningNum() const
{
	return WarningPackagePathnames.Num();
}

TMap<FString, FString> FUnsavedAssetsTracker::GetWarnings() const
{
	TMap<FString, FString> Warnings;
	for (const FString& PackagePathname : WarningPackagePathnames)
	{
		if (const FStatus* Status = UnsavedPackages.Find(PackagePathname))
		{
			Warnings.Add(Status->AbsPackageFilePathname, Status->WarningText.ToString());
		}
		else
		{
			checkNoEntry(); // The lists are out of sync.
		}
	}

	return Warnings;
}

bool FUnsavedAssetsTracker::ShouldTrackDirtyPackage(const UPackage* Package)
{
	// Ignore the packages that aren't meant to be persistent.
	if (!IsPackageMeantToBeSaved(Package))
	{
		return false;
	}
	else if (FPackageName::IsTempPackage(Package->GetPathName()))
	{
		// When working on an 'Untitled' map, the map is in a temp packages until saved. For a OFPA map,
		// added actors to the temp map are counted dirty, but when saving, the engine only propose to save
		// the dirty worlds which create a desync in numbers displayed by the widget and the save dialog. For
		// simplicity, when we have dirty temp package, just look at the source of truth and sync with it.
		bSyncWithDirtyPackageList = true;
		return false;
	}
	
	return true;
}

void FUnsavedAssetsTracker::OnPackageMarkedDirty(UPackage* Package, bool bWasDirty)
{
	if (ShouldTrackDirtyPackage(Package))
	{
		if (IsInGameThread())
		{
			StartTrackingDirtyPackage(Package);
		}
		else
		{
			FScopeLock Lock(&DirtyNotificationLock);
			DirtyPackageEventsReportedOnBackgroundThread.Emplace(Package->GetPathName(), Package);
		}
	}
}

void FUnsavedAssetsTracker::OnPackageDirtyStateUpdated(UPackage* Package)
{
	if (!ShouldTrackDirtyPackage(Package))
	{
		return;
	}

	if (IsInGameThread())
	{
		if (Package->IsDirty())
		{
			StartTrackingDirtyPackage(Package);
		}
		else
		{
			StopTrackingDirtyPackage(Package->GetPathName());
		}
	}
	else
	{
		FScopeLock Lock(&DirtyNotificationLock);
		DirtyPackageEventsReportedOnBackgroundThread.Emplace(Package->GetPathName(), Package);
	}
}

void FUnsavedAssetsTracker::OnPackageSaved(const FString& PackagePathname, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	if (ObjectSaveContext.IsProceduralSave() || ObjectSaveContext.IsFromAutoSave())
	{
		return; // Don't track procedural save (during cooking) nor package auto-saved as backup in case of crash
	}

	if (ShouldTrackDirtyPackage(Package))
	{
		StopTrackingDirtyPackage(Package->GetPathName());
	}
}

void FUnsavedAssetsTracker::OnPackageDeleted(UPackage* Package)
{
	if (UnsavedPackages.Contains(Package->GetPathName()))
	{
		bSyncWithDirtyPackageList = true;
	}
	RemoveTedsPackage(Package->GetPathName());
}

void FUnsavedAssetsTracker::OnPostGarbageCollect()
{
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::OnActorDeleted(AActor* Actor)
{
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::SyncWithDirtyPackageList()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnsavedAssetsTracker::SyncWithDirtyPackageList);

	// The list of dirty packages tracked by the engine (considered source of truth)
	TArray<UPackage*> DirtyPackages;
	FEditorFileUtils::GetDirtyPackages(DirtyPackages);

	// Convert the dirty packages pathname only once.
	TSet<FString> DirtyPackagePathnames;
	DirtyPackagePathnames.Reserve(DirtyPackages.Num());
	for (UPackage* DirtyPackage : DirtyPackages)
	{
		DirtyPackagePathnames.Add(DirtyPackage->GetPathName());
	}

	TArray<FString> ToRemove;

	// Remove packages that used to be dirty but aren't dirty anymore. (usually because the package was saved/renamed at the same time)
	for (const TPair<FString, FStatus>& PackagePathnameStatusPair : UnsavedPackages)
	{
		if (!DirtyPackagePathnames.Contains(PackagePathnameStatusPair.Key))
		{
			ToRemove.Emplace(PackagePathnameStatusPair.Key);
		}
	}
	for (const FString& PackagePathnameToRemove : ToRemove)
	{
		StopTrackingDirtyPackage(PackagePathnameToRemove);
	}

	// Add packages that aren't tracked yet
	for (const UPackage* Package : DirtyPackages)
	{
		if (IsPackageMeantToBeSaved(Package))
		{
			StartTrackingDirtyPackage(Package); // This early out if the package is already tracked.
		}
	}
}

void FUnsavedAssetsTracker::OnUndo(const FTransactionContext& TransactionContext, bool Succeeded)
{
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::OnRedo(const FTransactionContext& TransactionContext, bool Succeeded)
{
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::OnWorldPostRename(UWorld* InWorld)
{
	// Saving the temporary 'Untitled' map into a package is a save/rename operation. It is simpler to
	// sync the list of dirty package rather than implementing the rename logic, but a bit less efficient.
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	// Changing map sometimes drop changes to the temporary 'Untitled' map. It is simpler to sync the
	// list of dirty package rather than implementing the map tear down logic, but a bit less efficient.
	bSyncWithDirtyPackageList = true;
}

void FUnsavedAssetsTracker::StartTrackingDirtyPackage(const UPackage* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnsavedAssetsTracker::StartTrackingDirtyPackage);

	FString PackagePathname = Package->GetPathName();
	if (UnsavedPackages.Contains(PackagePathname))
	{
		return; // Early out if the package is already tracked.
	}

	// The absolute package path on disk (That's a much slower operation).
	FString AbsPackageFilePathname = GetAbsolutePackageFilePathname(Package);
	if (AbsPackageFilePathname.IsEmpty())
	{
		return;
	}

	FStatus& Status = UnsavedPackages.Emplace(PackagePathname, FStatus(AbsPackageFilePathname));
	UnsavedAbsFilePathames.Emplace(AbsPackageFilePathname, PackagePathname); // For fast reverse lookup.

	if (!HasPackageWritePermissions(Package))
	{
		Status.WarningType = EWarningTypes::PackageWritePermission;
		Status.WarningText = FText::Format(LOCTEXT("Write_Permission_Warning", "Insufficient writing permission to save {0}"), FText::FromString(Package->GetName()));
		WarningPackagePathnames.Add(PackagePathname);
		ShowWarningNotificationIfNotAlreadyShown(EWarningTypes::PackageWritePermission, Status.WarningText);
	}

	ISourceControlModule::Get().GetSourceControlFileStatusMonitor().StartMonitoringFile(
		reinterpret_cast<uintptr_t>(this),
		AbsPackageFilePathname,
		FSourceControlFileStatusMonitor::FOnSourceControlFileStatus::CreateSP(this, &FUnsavedAssetsTracker::OnSourceControlFileStatusUpdate));

	UpdateTedsPackage(PackagePathname, true);

	FUnsavedAssetsTrackerModule::Get().OnUnsavedAssetAdded.Broadcast(AbsPackageFilePathname);

	UE_LOGF(LogUnsavedAssetsTracker, Verbose, "Added file to the unsaved asset list: %ls (%ls)", *GetHumanFriendlyAssetName(Package), *PackagePathname);
}

void FUnsavedAssetsTracker::StopTrackingDirtyPackage(const FString& PackagePathname)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnsavedAssetsTracker::StopTrackingDirtyPackage);

	if (FStatus* Status = UnsavedPackages.Find(PackagePathname))
	{
		FString AbsPackageFilePathname = MoveTemp(Status->AbsPackageFilePathname);

		UnsavedPackages.Remove(PackagePathname);
		WarningPackagePathnames.Remove(PackagePathname);
		UnsavedAbsFilePathames.Remove(AbsPackageFilePathname);

		UpdateTedsPackage(PackagePathname, false);

		ISourceControlModule::Get().GetSourceControlFileStatusMonitor().StopMonitoringFile(reinterpret_cast<uintptr_t>(this), AbsPackageFilePathname);
		FUnsavedAssetsTrackerModule::Get().OnUnsavedAssetRemoved.Broadcast(AbsPackageFilePathname);

		if (WarningPackagePathnames.IsEmpty())
		{
			ShownWarnings.Reset();
		}

		UE_LOGF(LogUnsavedAssetsTracker, Verbose, "Removed file from the unsaved asset list: %ls", *PackagePathname);
	}
}

void FUnsavedAssetsTracker::UpdateTedsPackage(const FString& PackagePathname, const bool bIsDirty)
{
	using namespace UE::Editor::DataStorage;

	if (bSyncUnsavedAssetInfoToTEDS)
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (FString PackagePath; FPackageName::DoesPackageExist(PackagePathname, &PackagePath))
			{
				FPaths::NormalizeFilename(PackagePath);
				FString FullPackageFilename = FPaths::ConvertRelativePathToFull(PackagePath);
				if (const RowHandle PackageRow = Storage->LookupMappedRow(UE::Editor::RevisionControl::MappingDomain, FMapKey(FullPackageFilename)); Storage->IsRowAvailable(PackageRow))
				{
					TArray<RowHandle> ChildrenRowHandles; // Rows that remain in the package, but need to sync IsDirty
					TArray<RowHandle> RemapChildrenRowHandles; // Rows that are no longer in the package and need to be resynced in the processor
					auto GetChildRows = [&ChildrenRowHandles, &RemapChildrenRowHandles, PackagePathname](const ICoreProvider& Context, RowHandle, RowHandle TargetRow)
					{
						if (const FTypedElementUObjectColumn* ObjectColumn = Context.GetColumn<FTypedElementUObjectColumn>(TargetRow))
						{
							if (const UObject* ObjectInstance = ObjectColumn->Object.Get(); ObjectInstance != nullptr)
							{
								// If the package of the object is null or isn't the same, we need to resync this object row to its current package
								if (const UPackage* Target = ObjectInstance->GetPackage())
								{
									FString Path;
									Target->GetPathName(nullptr, Path);
									if (!Path.Equals(PackagePathname))
									{
										RemapChildrenRowHandles.Add(TargetRow);
										return;
									}
								}
								else
								{
									RemapChildrenRowHandles.Add(TargetRow);
									return;
								}
							}
						}
						ChildrenRowHandles.Add(TargetRow);
					};
					Storage->WalkDepthFirst(Storage->FindHierarchyByName(PackageHierarchyName), PackageRow, GetChildRows);

					// Remove package columns from any rows that don't belong to this package anymore so they can resync
					Storage->BatchAddRemoveColumns(
						TConstArrayView<RowHandle>(RemapChildrenRowHandles),
						{},
						{FTypedElementPackageReference::StaticStruct(), FTedsPackageDirtyTag::StaticStruct(), FTedsPrimaryPackageObjectTag::StaticStruct()});
					

					// Any rows still in the package should update to the correct IsDirty state
					for (const RowHandle ChildRow : ChildrenRowHandles)
					{
						if (bIsDirty)
						{
							Storage->AddColumns<FTedsPackageDirtyTag>(ChildRow);
						}
						else
						{
							Storage->RemoveColumn<FTedsPackageDirtyTag>(ChildRow);
						}
					}
				}
			}
		}
	}
}

void FUnsavedAssetsTracker::RemoveTedsPackage(const FString& PackagePathname)
{
	using namespace UE::Editor::DataStorage;
    
	if (bSyncUnsavedAssetInfoToTEDS)
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (FString PackagePath; FPackageName::DoesPackageExist(PackagePathname, &PackagePath))
			{
				FPaths::NormalizeFilename(PackagePath);
				FString FullPackageFilename = FPaths::ConvertRelativePathToFull(PackagePath);
				if (const RowHandle PackageRow = Storage->LookupMappedRow(UE::Editor::RevisionControl::MappingDomain, FMapKey(FullPackageFilename)); Storage->IsRowAvailable(PackageRow))
				{
					TArray<RowHandle> ChildRowHandles;
					auto GetChildRows = [&ChildRowHandles](const ICoreProvider&, RowHandle, RowHandle TargetRow)
						{
							ChildRowHandles.Add(TargetRow);
						};
					Storage->WalkDepthFirst(Storage->FindHierarchyByName(PackageHierarchyName), PackageRow, GetChildRows);

					Storage->BatchAddRemoveColumns(
						TConstArrayView<RowHandle>(ChildRowHandles),
						{},
						{FTypedElementPackageReference::StaticStruct(), FTedsPackageDirtyTag::StaticStruct(), FTedsPrimaryPackageObjectTag::StaticStruct()});
					
					Storage->RemoveRow(PackageRow);
				}
			}
		}
	}
}

void FUnsavedAssetsTracker::OnSourceControlFileStatusUpdate(const FString& AbsFilePathname, const ISourceControlState* State)
{
	auto DiscardWarning = [this](const FString& InPackagePathname, FStatus& InStatus)
	{
		// Source control status update cannot clear the package write permission warning.
		if (InStatus.WarningType != EWarningTypes::PackageWritePermission)
		{
			WarningPackagePathnames.Remove(InPackagePathname);
			InStatus.WarningText = FText::GetEmpty();
			InStatus.WarningType = EWarningTypes::None;
		}

		if (WarningPackagePathnames.IsEmpty()) // All warning were cleared.
		{
			ShownWarnings.Reset(); // Reeactivate the notification next time a warning happens.
		}
	};

	if (const FString* PackagePathname = UnsavedAbsFilePathames.Find(AbsFilePathname)) // Reverse lookup.
	{
		if (FStatus* Status = UnsavedPackages.Find(*PackagePathname))
		{
			if (Status->WarningType == EWarningTypes::PackageWritePermission)
			{
				return; // Write permission issue has more weight than source control issue.
			}
			else if (State == nullptr) // Source control state was reset. (Changing source control provider/disabling source control)
			{
				DiscardWarning(*PackagePathname, *Status);
			}
			else if (TOptional<FText> WarningText = State->GetWarningText())
			{
				Status->WarningText = *WarningText;
				OnSourceControlWarningNotification(*State, *Status);
				WarningPackagePathnames.Emplace(*PackagePathname);
			}
			else
			{
				DiscardWarning(*PackagePathname, *Status);
			}
		}
		else
		{
			checkNoEntry(); // UnsavedAbsFilePathames values are keys in UnsavedPackages to perform reverse lookup and those must always match.
		}
	}
}

bool FUnsavedAssetsTracker::PromptToSavePackages()
{
	if (GetUnsavedAssetNum() > 0)
	{
		const bool bPromptUserToSave = true;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bClosingEditor = false;
		const bool bNotifyNoPackagesSaved = true;
		const bool bCanBeDeclined = false;
		if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined))
		{
			// User likely saved something, reset the warnings. We could scan the list of unsaved asset that weren't saved (if any) and check if some warning
			// types remain, but that looks overkill in this context.
			ShownWarnings.Reset();

			// Stay in sync with what the package the engine thinks is dirty.
			SyncWithDirtyPackageList();
			return true;
		}
	}
	return false;
}

bool FUnsavedAssetsTracker::IsAssetUnsaved(const FString& FileAbsPathname) const
{
	return UnsavedAbsFilePathames.Contains(FileAbsPathname);
}

void FUnsavedAssetsTracker::OnSourceControlWarningNotification(const ISourceControlState& State, FStatus& InOutStatus)
{
	auto UpdateAndShowWarningIfNotAlreadyShown = [this](EWarningTypes WarningType, const FText& Msg, FStatus& InOutStatus)
	{
		// Update the warning type.
		InOutStatus.WarningType = WarningType;
		ShowWarningNotificationIfNotAlreadyShown(WarningType, Msg);
	};

	if (State.IsConflicted())
	{
		UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::Conflicted, LOCTEXT("Conflicted_Warning", "Warning: Assets you have edited have conflict(s)."), InOutStatus);
	}
	else if (!State.IsCurrent())
	{
		UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::OutOfDate, LOCTEXT("Out_of_Date_Warning", "Warning: Assets you have edited are out of date."), InOutStatus);
	}
	else if (State.IsCheckedOutOther())
	{
		UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::CheckedOutByOther, LOCTEXT("Locked_by_Other_Warning", "Warning: Assets you have edited are locked by another user."), InOutStatus);
	}
	else if (!State.IsCheckedOut())
	{
		if (State.IsCheckedOutInOtherBranch())
		{
			UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::CheckedOutInOtherBranch, LOCTEXT("Checked_Out_In_Other_Branch_Warning", "Warning: Assets you have edited are checked out in another branch."), InOutStatus);
		}
		else if (State.IsModifiedInOtherBranch())
		{
			UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::ModifiedInOtherBranch, LOCTEXT("Modified_In_Other_Branch_Warning", "Warning: Assets you have edited are modified in another branch."), InOutStatus);
		}
	}
	else if (State.GetWarningText().IsSet())
	{
		UpdateAndShowWarningIfNotAlreadyShown(EWarningTypes::Other, LOCTEXT("Generic_Warning", "Warning: Assets you have edited have warnings."), InOutStatus);
	}
}

void FUnsavedAssetsTracker::ShowWarningNotificationIfNotAlreadyShown(EWarningTypes WarningType, const FText& Msg)
{
	// Show the notification if it hasn't been shown since the last reset/save.
	if (bWarningNotificationEnabled && !ShownWarnings.Contains(WarningType))
	{
		// Setup the notification for operation feedback
		FNotificationInfo Info(Msg);
		Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		Notification->SetCompletionState(SNotificationItem::CS_None);
		ShownWarnings.Add(WarningType);
	}
}

bool FUnsavedAssetsTracker::Tick(float DeltaTime)
{
	if (DirtyNotificationLock.TryLock()) // Don't block the main thread.
	{
		for (TPair<FString, TWeakObjectPtr<UPackage>>& PathNamePackagePair : DirtyPackageEventsReportedOnBackgroundThread)
		{
			if (PathNamePackagePair.Value.IsValid())
			{
				UPackage* Package = PathNamePackagePair.Value.Get();
				if (Package->IsDirty())
				{
					StartTrackingDirtyPackage(Package);
				}
				else
				{
					StopTrackingDirtyPackage(Package->GetPathName());
				}
			}
		}

		DirtyPackageEventsReportedOnBackgroundThread.Reset();
		DirtyNotificationLock.Unlock();
	}

	if (bSyncWithDirtyPackageList)
	{
		if (GEditor && GEditor->GetPIEWorldContext() != nullptr)
		{
			return true; // delay update until PIE ends
		}
		
		SyncWithDirtyPackageList();
		bSyncWithDirtyPackageList = false;
	}

	return true; // Tick again.
}

#undef LOCTEXT_NAMESPACE
