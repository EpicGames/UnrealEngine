// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "DataStorage/Handles.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectSaveContext.h"
#include "UnrealEdMisc.h"

class AActor;
class ISourceControlState;
class UPackage;
class UWorld;
struct FTransactionContext;

/**
 * Tracks assets that has in-memory modification not saved to disk yet and checks
 * the source control states of those assets when a source control provider is available.
 */
class FUnsavedAssetsTracker : public TSharedFromThis<FUnsavedAssetsTracker>
{
public:
	FUnsavedAssetsTracker();
	virtual ~FUnsavedAssetsTracker();

	/** Returns the number of unsaved assets currently tracked. */
	int32 GetUnsavedAssetNum() const;

	/** Returns the list of unsaved assets. */
	TArray<FString> GetUnsavedAssets() const;

	/** Returns the number of active warnings. */
	int32 GetWarningNum() const;

	/** Returns a map of pathname/warnings. */
	TMap<FString, FString> GetWarnings() const;

	/** Sets whether a toast notification is emitted when looking up the source control state of an unsaved asset generates a warning. */
	void SetWarningNotificationEnabled(bool bEnabled) { bWarningNotificationEnabled = bEnabled; }

	/** Displays a dialog prompting the user to save the packages. */
	bool PromptToSavePackages();

	/** Check if the input asset is unsaved. */
	bool IsAssetUnsaved(const FString& FileAbsPathname) const;

private:
	enum class EWarningTypes
	{
		None,
		Conflicted,
		OutOfDate,
		CheckedOutByOther,
		CheckedOutInOtherBranch,
		ModifiedInOtherBranch,
		PackageWritePermission,
		Other,
	};

	struct FStatus
	{
		FStatus(const FString& InAbsFilePathname) : AbsPackageFilePathname(InAbsFilePathname) { }
		FString AbsPackageFilePathname; // On disk.
		EWarningTypes WarningType = EWarningTypes::None;
		FText WarningText;
	};

	/**
	 * Invoked when a package is marked dirty.
	 *
	 * @param Pkg The package that was marked as dirty
	 * @param bWasDirty Whether the package was previously dirty before the call to MarkPackageDirty
	 */
	void OnPackageMarkedDirty(UPackage* Package, bool bWasDirty);

	/**
	 * Invoked when a package is saved.
	 *
	 * @param Filename The filename the package was saved to
	 * @param Package The package that was saved
	 * @param ObjectSaveContext The package saving context.
	 */
	void OnPackageSaved(const FString& PackagePathname, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);

	/**
	 * Invoked when the dirty state of a package changed. Used to track when a package was cleaned without being saved.
	 */
	void OnPackageDirtyStateUpdated(UPackage* Package);

	/**
	 * Invoked when the level editor changes map. Used to track when a temporary 'Untitled' unsaved map is closed without saving.
	 */
	void OnMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType);

	/**
	 * Invoked when the undo stack changes. Used to track when the undo/redo affect the dirty state of a package.
	 */
	void OnUndo(const FTransactionContext& TransactionContext, bool Succeeded);
	void OnRedo(const FTransactionContext& TransactionContext, bool Succeeded);

	/** Invoked after garbage collection. */
	void OnPostGarbageCollect();

	/** Invoked when a package is deleted. */
	void OnPackageDeleted(UPackage* Package);

	/** Invoked when an actor is deleted. */
	void OnActorDeleted(AActor* Actor);

	/** Invoked when a world is successfully renamed. Used to track when a temporary 'Untitled' unsaved map is saved with a new name. */
	void OnWorldPostRename(UWorld* World);

	/** Starts to track an unsaved package.*/
	void StartTrackingDirtyPackage(const UPackage* Package);

	/** Stops tracking the specified unsaved package if the package was previously tracked. */
	void StopTrackingDirtyPackage(const FString& PackagePathname);

	/** Invoked when the status of a source controlled is updated, with the corresponding warning, if any. */
	void OnSourceControlFileStatusUpdate(const FString& Pathname, const ISourceControlState* Status);

	/** Invoked when a new warning is discovered. */
	void OnSourceControlWarningNotification(const ISourceControlState& State, FStatus& InOutStatus);

	/** Show a toast notifications if the warning types hasn't been shown yet. */
	void ShowWarningNotificationIfNotAlreadyShown(EWarningTypes WarningType, const FText& Msg);

	/** Refresh the list of unsaved files from the list returned by FEditorFileUtils::GetDirtyPackages(). */
	void SyncWithDirtyPackageList();

	/** Invoked by the engine on the game thread. */
	bool Tick(float DeltaTime);

	/** Whether the specified package should be tracked. */
	bool ShouldTrackDirtyPackage(const UPackage* Package);
	
	/** Adds the Dirty tag to the Package and its contained object rows */
	static void UpdateTedsPackage(const FString& PackagePathname, const bool bIsDirty);
	
	/** Removes the Package row from TEDS */
	static void RemoveTedsPackage(const FString& PackagePathname);

private:
	/** Supports multithreaded Dirty notifications. */
	FCriticalSection DirtyNotificationLock;

	/** Keep the packages that were marked/unmarked dirty from a background thread to check the status on the game thread. */
	TMap<FString, TWeakObjectPtr<UPackage>> DirtyPackageEventsReportedOnBackgroundThread;

	/** Maps the package pathname to the status */
	TMap<FString, FStatus> UnsavedPackages;

	/** Maps the packages absolute file name (on file system) to the package pathname for fast reverse lookup. */
	TMap<FString, FString> UnsavedAbsFilePathames;

	/** Packages pathname for which a warning was issued. */
	TSet<FString> WarningPackagePathnames;

	/** Sets of warning shown to the user. */
	TSet<EWarningTypes> ShownWarnings;

	/** Handle to unregister the tick delegate. */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Whether toast notification should be shown when a warning is detected. */
	bool bWarningNotificationEnabled = true;

	/** Whether the tracker should sync its dirty packages list with what the engine considers dirty. */
	bool bSyncWithDirtyPackageList = false;
};
