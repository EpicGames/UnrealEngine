// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsavedAssetsTrackerModule.h"

#include "CoreGlobals.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "Misc/App.h"
#include "UnsavedAssetsTracker.h"
#include "SUnsavedAssetsStatusBarWidget.h"
#include "UnsavedAssetsAutoCheckout.h"
#include "Widgets/SNullWidget.h"

void FUnsavedAssetsTrackerModule::StartupModule()
{
	UnsavedAssetTracker = MakeShared<FUnsavedAssetsTracker>();
	UnsavedAssetAutoCheckout = MakeShared<FUnsavedAssetsAutoCheckout>(this);
}

void FUnsavedAssetsTrackerModule::ShutdownModule()
{
	UnsavedAssetTracker.Reset();
}

int32 FUnsavedAssetsTrackerModule::GetUnsavedAssetNum() const
{
	if (UnsavedAssetTracker)
	{
		return UnsavedAssetTracker->GetUnsavedAssetNum();
	}
	return 0;
}

TArray<FString> FUnsavedAssetsTrackerModule::GetUnsavedAssets() const
{
	if (UnsavedAssetTracker)
	{
		return UnsavedAssetTracker->GetUnsavedAssets();
	}
	return TArray<FString>();
}

bool FUnsavedAssetsTrackerModule::IsAssetUnsaved(const FString& FileAbsPathname) const
{
	if (UnsavedAssetTracker)
	{
		return UnsavedAssetTracker->IsAssetUnsaved(FileAbsPathname);
	}
	return false;
}

TSharedRef<SWidget> FUnsavedAssetsTrackerModule::MakeUnsavedAssetsStatusBarWidget()
{
	if (!UnsavedAssetTracker)
	{
		return SNullWidget::NullWidget;
	}

	TWeakPtr<FUnsavedAssetsTracker> WeakUnsavedAssetTracker(UnsavedAssetTracker);
	return SNew(SUnsavedAssetsStatusBarWidget, UnsavedAssetTracker)
		.OnClicked_Lambda([WeakUnsavedAssetTracker]()
		{
			if (TSharedPtr<FUnsavedAssetsTracker> Tracker = WeakUnsavedAssetTracker.Pin())
			{
				Tracker->PromptToSavePackages();
			}
			return FReply::Handled();
		});
}

bool FUnsavedAssetsTrackerModule::PromptToSavePackages()
{
	if (!UnsavedAssetTracker)
	{
		return false;
	}

	return UnsavedAssetTracker->PromptToSavePackages();
}

IMPLEMENT_MODULE(FUnsavedAssetsTrackerModule, UnsavedAssetsTracker)

