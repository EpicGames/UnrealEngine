// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "ISourceControlOperation.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectSaveContext.h"

#define UE_API UNSAVEDASSETSTRACKER_API

class FUnsavedAssetsTracker;
class FUnsavedAssetsAutoCheckout;
class SWidget;

/** Invoked when an asset was modified in memory and added to the tracker list. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUnsavedAssetAdded, const FString& /*FileAbsPathname*/);

/** Invoked when an asset modified in memory was saved and removed from the tracker list. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUnsavedAssetRemoved, const FString& /*FileAbsPathname*/);

/** Invoked before an asset is potentially automatically checked out of source control */
DECLARE_MULTICAST_DELEGATE_TwoParams(FPreUnsavedAssetAutoCheckout, const FString& /*FileAbsPathname*/, FSourceControlOperationRef& /*CheckoutOperation*/);

/** Invoked after an asset has been checked out of source control */
DECLARE_MULTICAST_DELEGATE_TwoParams(FPostUnsavedAssetAutoCheckout, const FString& /*FileAbsPathname*/, FSourceControlOperationRef const& /*CheckoutOperation*/);

/**
 * Tracks assets that has in-memory modification not saved to disk yet and checks
 * the source control states of those assets when a source control provider is available.
 */
class FUnsavedAssetsTrackerModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/**
	 * Retrieve the module instance.
	 */
	static inline FUnsavedAssetsTrackerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FUnsavedAssetsTrackerModule>("UnsavedAssetsTracker");
	}

	/** Returns the number of unsaved assets currently tracked. */
	UE_API int32 GetUnsavedAssetNum() const;

	/** Returns the list of unsaved assets. */
	UE_API TArray<FString> GetUnsavedAssets() const;

	/** Check if the input asset is unsaved. */
	UE_API bool IsAssetUnsaved(const FString& FileAbsPathname) const;

	/** Construct the widget for the Editor status bar.*/
	UE_API TSharedRef<SWidget> MakeUnsavedAssetsStatusBarWidget();

	/** Displays a dialog prompting the user to save unsaved packages. */
	UE_API bool PromptToSavePackages();

	/** Invoked when a file is added to the unsaved list. */
	FOnUnsavedAssetAdded OnUnsavedAssetAdded;

	/** Invoked when a file is removed from the unsaved list. */
	FOnUnsavedAssetRemoved OnUnsavedAssetRemoved;

	/** Invoked before an asset is potentially automatically checked out of source control */
	/** Every notified File/Operation combination is followed by one of the Cancel/Success/Failure notifications below */
	FPreUnsavedAssetAutoCheckout PreUnsavedAssetAutoCheckout;

	/** Invoked after an asset has been cancelled for automatic check out */
	FPostUnsavedAssetAutoCheckout PostUnsavedAssetAutoCheckoutCancel;
	
	/** Invoked after an asset has been checked out of source control */
	FPostUnsavedAssetAutoCheckout PostUnsavedAssetAutoCheckout;

	/** Invoked after an asset failed to be checked out of source control */
	FPostUnsavedAssetAutoCheckout PostUnsavedAssetAutoCheckoutFailure;
	
private:
	TSharedPtr<FUnsavedAssetsTracker> UnsavedAssetTracker;
	TSharedPtr<FUnsavedAssetsAutoCheckout> UnsavedAssetAutoCheckout;
};

#undef UE_API
